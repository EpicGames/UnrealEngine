// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISpeechAnimationSolver.h"



FString ISpeechAnimationSolver::GetLatestModelAssetPath()
{
	return "/StreamingADA/xsada_face_base_fp32_v2_0_0.xsada_face_base_fp32_v2_0_0";
}
