// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"



class FSpeechAnimationSolverModule : public IModuleInterface
{
public:
};

IMPLEMENT_MODULE(FSpeechAnimationSolverModule, SpeechAnimationSolver)
