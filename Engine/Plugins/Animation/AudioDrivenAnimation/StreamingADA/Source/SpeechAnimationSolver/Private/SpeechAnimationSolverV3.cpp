// Copyright Epic Games, Inc. All Rights Reserved.

#include "SpeechAnimationSolverV3.h"

#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeGPU.h"
#include "NNERuntimeCPU.h"

DEFINE_LOG_CATEGORY_STATIC(LogSpeechAnimationSolverV3, Log, All);



static TArray<FName> CurveNames = {
"CTRL_L_brow_down.ty",
"CTRL_R_brow_down.ty",
"CTRL_L_brow_lateral.ty",
"CTRL_R_brow_lateral.ty",
"CTRL_L_brow_raiseIn.ty",
"CTRL_R_brow_raiseIn.ty",
"CTRL_L_brow_raiseOut.ty",
"CTRL_R_brow_raiseOut.ty",
"CTRL_L_eye_blink.ty",
"CTRL_R_eye_blink.ty",
"CTRL_L_eye_squintInner.ty",
"CTRL_R_eye_squintInner.ty",
"CTRL_L_eye_cheekRaise.ty",
"CTRL_R_eye_cheekRaise.ty",
"CTRL_L_nose.ty",
"CTRL_R_nose.ty",
"CTRL_L_nose.tx",
"CTRL_R_nose.tx",
"CTRL_L_nose_nasolabialDeepen.ty",
"CTRL_R_nose_nasolabialDeepen.ty",
"CTRL_C_mouth.tx",
"CTRL_L_mouth_upperLipRaise.ty",
"CTRL_R_mouth_upperLipRaise.ty",
"CTRL_L_mouth_lowerLipDepress.ty",
"CTRL_R_mouth_lowerLipDepress.ty",
"CTRL_L_mouth_cornerPull.ty",
"CTRL_R_mouth_cornerPull.ty",
"CTRL_L_mouth_stretch.ty",
"CTRL_R_mouth_stretch.ty",
"CTRL_L_mouth_dimple.ty",
"CTRL_R_mouth_dimple.ty",
"CTRL_L_mouth_cornerDepress.ty",
"CTRL_R_mouth_cornerDepress.ty",
"CTRL_L_mouth_purseU.ty",
"CTRL_R_mouth_purseU.ty",
"CTRL_L_mouth_purseD.ty",
"CTRL_R_mouth_purseD.ty",
"CTRL_L_mouth_towardsU.ty",
"CTRL_R_mouth_towardsU.ty",
"CTRL_L_mouth_towardsD.ty",
"CTRL_R_mouth_towardsD.ty",
"CTRL_L_mouth_funnelU.ty",
"CTRL_R_mouth_funnelU.ty",
"CTRL_L_mouth_funnelD.ty",
"CTRL_R_mouth_funnelD.ty",
"CTRL_L_mouth_lipsTogetherU.ty",
"CTRL_R_mouth_lipsTogetherU.ty",
"CTRL_L_mouth_lipsTogetherD.ty",
"CTRL_R_mouth_lipsTogetherD.ty",
"CTRL_L_mouth_lipBiteU.ty",
"CTRL_R_mouth_lipBiteU.ty",
"CTRL_L_mouth_lipBiteD.ty",
"CTRL_R_mouth_lipBiteD.ty",
"CTRL_L_mouth_sharpCornerPull.ty",
"CTRL_R_mouth_sharpCornerPull.ty",
"CTRL_L_mouth_pushPullU.ty",
"CTRL_R_mouth_pushPullU.ty",
"CTRL_L_mouth_pushPullD.ty",
"CTRL_R_mouth_pushPullD.ty",
"CTRL_L_mouth_cornerSharpnessU.ty",
"CTRL_R_mouth_cornerSharpnessU.ty",
"CTRL_L_mouth_cornerSharpnessD.ty",
"CTRL_R_mouth_cornerSharpnessD.ty",
"CTRL_L_mouth_lipsRollU.ty",
"CTRL_R_mouth_lipsRollU.ty",
"CTRL_L_mouth_lipsRollD.ty",
"CTRL_R_mouth_lipsRollD.ty",
"CTRL_C_jaw.ty",
"CTRL_C_jaw.tx",
"CTRL_C_jaw_fwdBack.ty",
"CTRL_L_jaw_ChinRaiseD.ty",
"CTRL_R_jaw_ChinRaiseD.ty",
"CTRL_C_tongue_move.ty",
"CTRL_C_tongue_move.tx",
"CTRL_C_tongue_inOut.ty",
"CTRL_C_tongue_tipMove.ty",
"CTRL_C_tongue_tipMove.tx",
"CTRL_C_tongue_wideNarrow.ty",
"CTRL_C_tongue_press.ty",
"CTRL_C_tongue_roll.ty",
"CTRL_C_tongue_thickThin.ty"
};

FSpeechAnimationSolverV3::FSpeechAnimationSolverV3(const TObjectPtr<UNNEModelData> InModelData, const FString& InNNEBackend)
{
	check(CurveNames.Num() == 81);

	ModelData = InModelData;
	NNEBackend = InNNEBackend;
}

bool FSpeechAnimationSolverV3::Initialize()
{
	using namespace UE::NNE;

	if (!ModelData)
	{
		UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to load streaming speech to animation model");
		return false;
	}

	if (NNEBackend == "NNERuntimeORTDml") // GPU cases
	{
		TWeakInterfacePtr<INNERuntimeGPU> Runtime = GetRuntime<INNERuntimeGPU>(NNEBackend);
		if (!Runtime.IsValid())
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to get %ls runtime", *NNEBackend);
			return false;
		}

		TSharedPtr<IModelGPU> GPUModel = Runtime->CreateModelGPU(ModelData);
		if (!GPUModel)
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to create GPU model");
			return false;
		}

		Model = GPUModel->CreateModelInstanceGPU();
	}
	else // CPU cases
	{
		TWeakInterfacePtr<INNERuntimeCPU> Runtime = GetRuntime<INNERuntimeCPU>(NNEBackend);
		if (!Runtime.IsValid())
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to get %ls runtime", *NNEBackend);
			return false;
		}

		TSharedPtr<IModelCPU> CPUModel = Runtime->CreateModelCPU(ModelData);
		if (!CPUModel)
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to create CPU model");
			return false;
		}

		Model = CPUModel->CreateModelInstanceCPU();
	}

	if (!Model)
	{
		UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to create streaming speech to animation model");
		return false;
	}

	constexpr int32 BatchSize = 1;

	FTensorShape TensorShape1 = FTensorShape::Make({ BatchSize, 16000 });
	FTensorShape TensorShape2 = FTensorShape::Make({ BatchSize, 81 });
	FTensorShape TensorShape3 = FTensorShape::Make({ BatchSize, 21 });
	FTensorShape TensorShape4 = FTensorShape::Make({ BatchSize });
	FTensorShape TensorShape5 = FTensorShape::Make({ BatchSize });

	if (Model->SetInputTensorShapes({ TensorShape1, TensorShape2, TensorShape3, TensorShape4, TensorShape5 }) != UE::NNE::EResultStatus::Ok)
	{
		UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to set model input");
		Model.Reset();
		return false;
	}

	ClearCache();

	return true;
}

void FSpeechAnimationSolverV3::ClearCache()
{
	InputBuffer.Reset();
	AudioBuffer.SetNumZeroed(16000); // The first input to the model is 1.0s of audio (=16000 mono samples at 16kHz). Fill with zeros so first second of audio is not dead animation.
	CurveValues.SetNumZeroed(81); // The second input to the model is state of the 81 curves from the previous iteration. Initially zero
	Step.SetNumZeroed(1); // The 4th input to the model is a step counter that increments every frame. Initially zero.
	FrameBuffer.SetNumUninitialized(320); // A buffer to hold the 20ms (=320 mono samples at 16kHz) of audio required to advance the model to the next interaction

	AnimOut = FSpeechAnimationFrameData();
}

bool FSpeechAnimationSolverV3::SolveAudioFrame(const FSpeechAnimationAudioFrame& InAudioFrame, FSpeechAnimationFrameData& OutFrameData)
{
	const FSpeechAnimationAudioFrame* AudioFramePtr = nullptr;

	if (!ReformatAudio(InAudioFrame, AudioFramePtr))
	{
		return false;
	}

	if (!AudioFramePtr->bContiguous)
	{
		InputBuffer.Reset();
		AudioBuffer.SetNumZeroed(16000);
		CurveValues.SetNumZeroed(81);
		Step.SetNumZeroed(1);
		FrameBuffer.SetNumUninitialized(320);
	}

	InputBuffer.Append(AudioFramePtr->AudioSamples);

	while (InputBuffer.Num() >= 320) // Do we have enough data to pass to the speech solve
	{
		FMemory::Memcpy(FrameBuffer.GetData(), InputBuffer.GetData(), 320 * sizeof(float)); // Copy first 320 samples from InputBuffer into FrameBuffer

		InputBuffer = TArray<float>(&InputBuffer.GetData()[320], InputBuffer.Num() - 320); // The remaining samples becomes the new InputBuffer

		if (AudioBuffer.Num() == 16000) // Do we have the required 16000 sample audio buffer to run the next interaction of the speech solver. Currently always true to avoid initial deadtime but that may change.
		{
			// Yes, shift AudioBuffer down and add FrameBuffer to end of AudioBuffer to maintain 16000 samples
			FMemory::Memmove(AudioBuffer.GetData(), &AudioBuffer.GetData()[320], (16000 - 320) * sizeof(float));
			FMemory::Memcpy(&AudioBuffer.GetData()[16000 - 320], FrameBuffer.GetData(), 320 * sizeof(float));

			// Prepare the mood input array
			TArray<float, TAlignedHeapAllocator<64>> MoodArray;
			MoodArray.SetNumZeroed(21);
			MoodArray[(uint8) AudioFramePtr->Mood] = AudioFramePtr->MoodIntensity;

			// Prepare the lookahead array
			TArray<int64, TAlignedHeapAllocator<64>> LookaheadArray;
			LookaheadArray.SetNumZeroed(1);
			LookaheadArray[0] = AudioFramePtr->Lookahead / 20; // lookahead needs to be in frames, 1 frame = 20ms

			// Run the speech solver
			TArray<UE::NNE::FTensorBindingCPU> Inputs, Outputs;
			Inputs = { {(void*)AudioBuffer.GetData(), AudioBuffer.Num() * sizeof(float)}, {(void*)CurveValues.GetData(), CurveValues.Num() * sizeof(float)}, {(void*)MoodArray.GetData(), MoodArray.Num() * sizeof(float)}, {(void*)LookaheadArray.GetData(), LookaheadArray.Num() * sizeof(int64)}, {(void*)Step.GetData(), Step.Num() * sizeof(int64)} };
			Outputs = { {(void*)CurveValues.GetData(), CurveValues.Num() * sizeof(float)}, {(void*)Step.GetData(), Step.Num() * sizeof(int64)} };

			if (Model->RunSync(Inputs, Outputs) != UE::NNE::EResultStatus::Ok)
			{
				UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Failed to run");
				return false;
			}

			check(CurveNames.Num() == CurveValues.Num());

			AnimOut.AudioFrame = InAudioFrame;
			AnimOut.CurveNames = CurveNames;
			AnimOut.CurveValues = CurveValues;
			AnimOut.SolvedTime = FPlatformTime::Seconds();
		}
		else
		{
			// No, keep accumulating FrameBuffer in AudioBuffer
			AudioBuffer.Append(FrameBuffer);
		}
	}

	OutFrameData = AnimOut;

	return true;
}

int32 FSpeechAnimationSolverV3::GetNumCurves()
{
	return CurveNames.Num();
}

const TArray<FName>& FSpeechAnimationSolverV3::GetCurveNames() const
{
	return CurveNames;
}

bool FSpeechAnimationSolverV3::ReformatAudio(const FSpeechAnimationAudioFrame& InAudioFrame, const FSpeechAnimationAudioFrame*& OutAudioFramePtr)
{
	const int32 NumChannels = 1;
	const int32 SampleRate = 16000;

	if (InAudioFrame.NumChannels == NumChannels && InAudioFrame.SampleRate == SampleRate)
	{
		OutAudioFramePtr = &InAudioFrame;
		return true;
	}

	AudioFrameReformatted = InAudioFrame;
	OutAudioFramePtr = &AudioFrameReformatted;

	if (AudioFrameReformatted.NumChannels != NumChannels)
	{
		if (AudioFrameReformatted.NumChannels == 2 && NumChannels == 1)
		{
			TArray<float> Mono;
			Mono.SetNumUninitialized(AudioFrameReformatted.SamplesCount);

			const float* StereoData = AudioFrameReformatted.AudioSamples.GetData();
			float* MonoData = Mono.GetData();

			for (int32 Index = 0; Index < AudioFrameReformatted.SamplesCount; ++Index, StereoData += 2, ++MonoData)
			{
				*MonoData = (StereoData[0] + StereoData[1]) / 2;
			}

			AudioFrameReformatted.NumChannels = 1;
			AudioFrameReformatted.AudioSamples = Mono;
		}
		else
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Unsupported channel mix - have %i want %i", AudioFrameReformatted.NumChannels, NumChannels);
			return false;
		}
	}

	if (AudioFrameReformatted.SampleRate != SampleRate)
	{
		if (AudioFrameReformatted.SampleRate <= 0)
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Sample rate too low");
			return false;
		}

		if (!bResamplerInitialized)
		{
			Resampler.Init(Audio::EResamplingMethod::ZeroOrderHold, float(SampleRate) / AudioFrameReformatted.SampleRate, 1);
			bResamplerInitialized = true;
		}

		TArray<float> ResampledBuffer;
		const int32 ResampledSamplesCount = (float)AudioFrameReformatted.SamplesCount / (float)AudioFrameReformatted.SampleRate * SampleRate;
		ResampledBuffer.SetNumZeroed(ResampledSamplesCount);

		int32 ResampledNumFrames = 0;
		if (Resampler.ProcessAudio(AudioFrameReformatted.AudioSamples.GetData(), AudioFrameReformatted.SamplesCount, true, ResampledBuffer.GetData(), ResampledSamplesCount, ResampledNumFrames) == 0)
		{
			AudioFrameReformatted.SampleRate = SampleRate;
			AudioFrameReformatted.SamplesCount = ResampledSamplesCount;
			AudioFrameReformatted.AudioSamples = ResampledBuffer;
		}
		else
		{
			UE_LOGF(LogSpeechAnimationSolverV3, Warning, "Resampling failed");
			return false;
		}
	}

	return true;
}
