// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API SPEECHANIMATIONSOLVER_API

struct FSpeechAnimationFrameData;
struct FSpeechAnimationAudioFrame;



/**
 * The interface to implement a Speech Animation Solver
 */

class ISpeechAnimationSolver
{
public:
	virtual ~ISpeechAnimationSolver() = default;

	/** Initialize the solver */
	virtual bool Initialize() = 0;

	/** Clear the solver data cache to avoid having to restart it between multiple uses */
	virtual void ClearCache() = 0;
	
	/** Solve the given audio frame to control rig channels and/or regular curves */
	virtual bool SolveAudioFrame(const FSpeechAnimationAudioFrame& InAudioFrame, FSpeechAnimationFrameData& OutFrameData) = 0;

	/** @returns the number of control rig curves to expect */
	virtual int32 GetNumCurves() = 0;

	/** @returns the name of the control rig curve channels to expect */
	virtual const TArray<FName>& GetCurveNames() const = 0;

	/** @returns the asset path of the most up to date model */
	UE_API static FString GetLatestModelAssetPath();
};

#undef UE_API
