// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SpeechAnimationSolverTypes.generated.h"



/**
 * The Speech Animation mood options.
 */
UENUM(BlueprintType)
enum class EAudioDrivenAnimationMood : uint8
{
	AutoDetect = 255 UMETA(DisplayName = "Auto Detect"),

	Neutral = 0 UMETA(DisplayName = "Neutral"),
	Happiness = 1 UMETA(DisplayName = "Happy"),
	Sadness = 2 UMETA(DisplayName = "Sad"),
	Disgust = 3 UMETA(DisplayName = "Disgust"),
	Anger = 4 UMETA(DisplayName = "Anger"),
	Surprise = 5 UMETA(DisplayName = "Surprise"),
	Fear = 6 UMETA(DisplayName = "Fear"),
	Confidence = 10 UMETA(DisplayName = "Confident"),
	Excitement = 14 UMETA(DisplayName = "Excited"),
	Boredom = 15 UMETA(DisplayName = "Bored"),
	Playfulness = 17 UMETA(DisplayName = "Playful"),
	Confusion = 19 UMETA(DisplayName = "Confused"),
};

/**
 * The Speech Animation solver's input frame data.
 */
USTRUCT(BlueprintType)
struct FSpeechAnimationAudioFrame
{
	GENERATED_BODY()

	/** The audio sample to solve */
	UPROPERTY(BlueprintReadWrite, Category = "Speech Animation")
	TArray<float> AudioSamples;

	/** The number of audio samples for this frame of audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	int32 SamplesCount = 0;

	/** The sample rate of this frame of audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	int32 SampleRate = 0;

	/** The number of channels for this frame of audio  */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	int32 NumChannels = 1;

	/** Whether this audio sample is contiguous with the last audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	bool bContiguous = true;

	/** The mood for this audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	EAudioDrivenAnimationMood Mood = EAudioDrivenAnimationMood::Neutral;

	/** The mood intensity for this audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	float MoodIntensity = 1;

	/** The lookahead for this audio */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	int32 Lookahead = 80;

	/** The timecode associated to this audio sample */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	double ArrivalTime = 0.f;

	/** The id for this audio frame */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Speech Animation")
	FGuid FrameId;
};

/**
 * The Speech Animation solver's output frame data.
 */
USTRUCT(BlueprintType)
struct FSpeechAnimationFrameData
{
	GENERATED_BODY()

	/** The name of the curves that were solved onto */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	TArray<FName> CurveNames;

	/** The values of the curves that were solved */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	TArray<float> CurveValues;

	/** The audio frame that was solved */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	FSpeechAnimationAudioFrame AudioFrame;

	/** The time at which the frame was actually solved */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Speech Animation")
	double SolvedTime = 0;
};