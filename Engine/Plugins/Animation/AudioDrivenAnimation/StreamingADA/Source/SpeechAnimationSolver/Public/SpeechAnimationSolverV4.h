// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISpeechAnimationSolver.h"
#include "SpeechAnimationSolverTypes.h"
#include "AudioResampler.h"

#define UE_API SPEECHANIMATIONSOLVER_API

namespace UE::NNE
{
	class IModelInstanceRunSync;
}



class FSpeechAnimationSolverV4 : public ISpeechAnimationSolver
{
public:

	UE_API FSpeechAnimationSolverV4(const TObjectPtr<class UNNEModelData> InModelData, const FString& InNNEBackend);

	/* ISpeechAnimationSolver */
	UE_API bool Initialize() override;
	UE_API void ClearCache() override;
	UE_API bool SolveAudioFrame(const FSpeechAnimationAudioFrame& InAudioFrame, FSpeechAnimationFrameData& OutFrameData) override;
	UE_API int32 GetNumCurves() override;
	UE_API const TArray<FName>& GetCurveNames() const override;
	/* ~ISpeechAnimationSolver */

private:

	TObjectPtr<class UNNEModelData> ModelData;
	FString NNEBackend;

	TSharedPtr<UE::NNE::IModelInstanceRunSync> Model = nullptr;

	TArray<float, TAlignedHeapAllocator<64>> AudioBuffer; // IREE case requires 64 byte alignment
	TArray<float, TAlignedHeapAllocator<64>> CurveValues;
	TArray<float, TAlignedHeapAllocator<64>> MoodState;
	TArray<int64, TAlignedHeapAllocator<64>> BlinkBound;
	TArray<int64, TAlignedHeapAllocator<64>> BlinkSel;
	TArray<int64, TAlignedHeapAllocator<64>> Step;

	TArray<float> InputBuffer;
	TArray<float> FrameBuffer;

	FSpeechAnimationFrameData AnimOut;

	bool ReformatAudio(const FSpeechAnimationAudioFrame& InAudioFrame, const FSpeechAnimationAudioFrame*& OutAudioFramePtr);
	FSpeechAnimationAudioFrame AudioFrameReformatted;
	Audio::FResampler Resampler;
	bool bResamplerInitialized = false;
};

#undef UE_API
