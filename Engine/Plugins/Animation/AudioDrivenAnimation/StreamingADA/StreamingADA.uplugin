{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Streaming Audio Driven Animation",
	"Description": "Streaming Audio Driven Animation",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Hidden":  true,
	"Plugins": [
		{
			"Name": "NNERuntimeORT",
			"Enabled": true
		},
		{
			"Name": "NNERuntimeBasicCpu",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "SpeechAnimationSolver",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"IsExperimentalVersion": false
}